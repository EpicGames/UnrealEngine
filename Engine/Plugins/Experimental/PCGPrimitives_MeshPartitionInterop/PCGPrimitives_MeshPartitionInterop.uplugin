{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PCG Primitives Mesh Partition Interop",
	"Description": "Extra PCG Primitives and Examples Library for Mesh Partition interop",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "PCGPrimitives",
			"Enabled": true
		},
		{
			"Name": "MeshPartition",
			"Enabled": true
		},
		{
			"Name": "MeshTerrainMode",
			"Enabled": true
		},
		{
			"Name": "PCGMeshPartitionInterop",
			"Enabled": true
		},
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "PCGBiomeSample",
			"Enabled": true
		}
	]
}